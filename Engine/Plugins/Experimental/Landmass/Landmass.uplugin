{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "Landmass",
	"Description": "",
	"Category": "Other",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"CanContainContent": true,
	"IsBetaVersion": true,
	"Installed": false,
	"Modules": [
		{
			"Name": "Landmass",
			"Type": "Runtime",
			"LoadingPhase": "Default"

		},
		{
			"Name": "LandmassEditor",
			"Type": "Editor",
			"LoadingPhase": "Default"
		}
	]
}