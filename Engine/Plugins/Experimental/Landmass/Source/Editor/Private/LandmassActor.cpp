// Copyright Epic Games, Inc. All Rights Reserved.

#include "LandmassActor.h"
#include "LandmassManagerBase.h"
#include "EngineUtils.h"
#include "Landscape.h"
#include "Components/StaticMeshComponent.h"
#include "Materials/MaterialInterface.h"
#include "Kismet/GameplayStatics.h"
#include "LevelEditor.h"
#include "Modules/ModuleManager.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LandmassActor)

ALandmassActor::ALandmassActor()
{
	USceneComponent* SceneComp = CreateDefaultSubobject<USceneComponent>(TEXT("RootComponent"));
	RootComponent = SceneComp;
	SceneComp->Mobility = EComponentMobility::Static;

	MeshExtentsQuad = CreateDefaultSubobject<UStaticMeshComponent>(TEXT("ExtentsMesh"));
	MeshExtentsQuad->SetupAttachment(RootComponent);
	MeshExtentsQuad->SetStaticMesh(Cast< UStaticMesh >(FSoftObjectPath(TEXT("StaticMesh'/Engine/ArtTools/RenderToTexture/Meshes/S_1_Unit_Plane.S_1_Unit_Plane'")).TryLoad()));
	MeshExtentsQuad->SetMaterial(0, Cast< UMaterialInterface >(FSoftObjectPath(TEXT("Material'/Landmass/Landscape/BlueprintBrushes/Materials/Internal/BrushBounds.BrushBounds'")).TryLoad()));
	MeshExtentsQuad->SetCollisionEnabled(ECollisionEnabled::NoCollision);

	BrushSpriteMesh = CreateDefaultSubobject<UStaticMeshComponent>(TEXT("BrushSpriteMesh"));
	BrushSpriteMesh->SetupAttachment(RootComponent);
	BrushSpriteMesh->SetStaticMesh(Cast< UStaticMesh >(FSoftObjectPath(TEXT("StaticMesh'/Landmass/Landscape/Meshes/S_SpritePlane.S_SpritePlane'")).TryLoad()));
	BrushSpriteMesh->SetMaterial(0, Cast< UMaterialInterface >(FSoftObjectPath(TEXT("Material'/Landmass/Landscape/BlueprintBrushes/Materials/Sprite/M_BrushSprite_01.M_BrushSprite_01'")).TryLoad()));
	BrushSpriteMesh->SetCollisionEnabled(ECollisionEnabled::NoCollision);
	BrushSpriteMesh->SetCastShadow(false);

	PrimaryActorTick.bCanEverTick = true;
	PrimaryActorTick.TickGroup = TG_DuringPhysics;
	PrimaryActorTick.bStartWithTickEnabled = true;
	PrimaryActorTick.SetTickFunctionEnable(true);

	if (!HasAnyFlags(EObjectFlags::RF_ClassDefaultObject))
	{
		FLevelEditorModule& LevelEditor = FModuleManager::GetModuleChecked<FLevelEditorModule>("LevelEditor");
		OnActorSelectionChangedHandle = LevelEditor.OnActorSelectionChanged().AddUObject(this, &ALandmassActor::HandleActorSelectionChanged);
	}
}

void ALandmassActor::OnConstruction(const FTransform& Transform) 
{
	Super::OnConstruction(Transform);
	UpdateBrushExtents();
	FindOrSpawnManager();
	MeshExtentsQuad->SetVisibility(bWasSelected);
	if (BrushManager != nullptr)
	{
		if (AffectsHeightmap || AffectsWeightmaps || AffectsVisibility)
		{
			BrushManager->RequestUpdateFromBrush(this);
		}
	}
	//Used for Erosion Brush to update when using Brush as a mask
	OnBrushUpdated.Broadcast();
}

void ALandmassActor::Tick(float DeltaSeconds)
{
	this->CustomTick(DeltaSeconds);
}

void ALandmassActor::CustomTick_Implementation(float DeltaSeconds)
{

}

bool ALandmassActor::ShouldTickIfViewportsOnly() const
{
	return EditorTickIsEnabled;
}

//Used so that the BPImplementableEvent "RenderLayer" does not have to be marked BlueprintCallable which adds the unwanted "Parent:FunctionCall" to the BP graph
void ALandmassActor::RenderLayer_Native(const FLandscapeBrushParameters& InParameters)
{
	BrushRenderParameters = InParameters;
	RenderLayer(InParameters);
}

void ALandmassActor::RenderLayer_Implementation(const FLandscapeBrushParameters& InParameters)
{

}

void ALandmassActor::UpdateBrushExtents()
{
	//Mesh Quad represents the Brush AABB, update its size and use it to calculate the extents
	MeshExtentsQuad->SetRelativeScale3D(FVector(BrushSize, BrushSize, BrushSize));
	FVector2D Origin = FVector2D(MeshExtentsQuad->Bounds.Origin.X, MeshExtentsQuad->Bounds.Origin.Y);
	FVector2D BoxExtent = FVector2D(MeshExtentsQuad->Bounds.BoxExtent.X, MeshExtentsQuad->Bounds.BoxExtent.Y);

	BrushExtents = FVector4(Origin - BoxExtent, Origin + BoxExtent);

	float SpriteMeshScale = (BrushSize)  / 8;
	BrushSpriteMesh->SetRelativeScale3D(FVector(SpriteMeshScale, SpriteMeshScale, SpriteMeshScale));

	if (DrawToEntireLandscape)
	{

	}

}

void ALandmassActor::FastPreviewMode()
{
	BrushManager->TogglePreviewMode(true);
}

void ALandmassActor::RestoreLandscapeEditing()
{
	BrushManager->TogglePreviewMode(false);
}

void ALandmassActor::MoveBrushUp()
{
	if (BrushManager != nullptr)
	{
		BrushManager->MoveBrushUp(this);
	}
}

void ALandmassActor::MoveBrushDown()
{
	if (BrushManager != nullptr)
	{
		BrushManager->MoveBrushDown(this);
	}
}

void ALandmassActor::MoveToTop()
{
	if (BrushManager != nullptr)
	{
		BrushManager->MoveBrushToTop(this);
	}
}

void ALandmassActor::MoveToBottom()
{
	if (BrushManager != nullptr)
	{
		BrushManager->MoveBrushToBottom(this);
	}
}

void ALandmassActor::FindOrSpawnManager()
{
#if WITH_EDITOR
	//If brush already has a manager, do nothing
	if (BrushManager != nullptr || this->HasAnyFlags(RF_Transient))
	{
		return;
	}

	if (GetWorld()->WorldType != EWorldType::Editor)
	{
		return;
	}

	TArray<AActor*> FoundActors;
	UGameplayStatics::GetAllActorsOfClass(Cast<UObject>(GetWorld()), ALandmassManagerBase::StaticClass(), FoundActors);

	if (FoundActors.Num() > 0)
	{
		BrushManager = Cast<ALandmassManagerBase>(FoundActors[0]);
	}
	else
	{
		FActorSpawnParameters SpawnParams;
		SpawnParams.bAllowDuringConstructionScript = true; 

		FName BlueprintPath = TEXT("Class'/Landmass/Landscape/BlueprintBrushes/LandmassBrushManager.LandmassBrushManager_C'");
		UClass* ManagerBPClass = Cast<UClass>(StaticLoadObject(UClass::StaticClass(), NULL, *BlueprintPath.ToString()));
		ALandmassManagerBase* SpawnedManager = GetWorld()->SpawnActor<ALandmassManagerBase>(ManagerBPClass, SpawnParams);

		BrushManager = SpawnedManager;
	}

	for (TActorIterator<ALandscape> LandscapeIterator(GetWorld()); LandscapeIterator; ++LandscapeIterator)
	{
		if (BrushManager && !LandscapeIterator->IsTemplate())
		{
			BrushManager->SetTargetLandscape(*LandscapeIterator);
			break;
		}
	}
#endif
}

void ALandmassActor::DrawBrushMaterial(UMaterialInterface* InMaterial)
{
	if (BrushManager != nullptr)
	{
		BrushManager->DrawBrushMaterial(this, InMaterial);
	}
}

void ALandmassActor::SetMeshExentsMaterial(UMaterialInterface* Material)
{
	if (Material != nullptr)
	{
		MeshExtentsQuad->SetMaterial(0, Material);
	}
}

void ALandmassActor::HandleActorSelectionChanged(const TArray<UObject*>& NewSelection, bool bForceRefresh)
{
	if (!IsTemplate())
	{
		bool bUpdateActor = false;
		if (bWasSelected && !NewSelection.Contains(this))
		{
			bWasSelected = false;
			bUpdateActor = true;
		}
		if (!bWasSelected && NewSelection.Contains(this))
		{
			bWasSelected = true;
			bUpdateActor = true;
		}
		if (bUpdateActor)
		{
			ActorSelectionChanged(bWasSelected);
			MeshExtentsQuad->SetVisibility(bWasSelected);
		}
	}
}

void ALandmassActor::ActorSelectionChanged_Implementation(bool bSelected)
{

}