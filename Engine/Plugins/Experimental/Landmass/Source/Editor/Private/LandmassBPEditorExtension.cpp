// Copyright Epic Games, Inc. All Rights Reserved.

#include "LandmassBPEditorExtension.h"

#include "LevelEditorViewport.h"
#include "Engine/Texture.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LandmassBPEditorExtension)

bool ULandmassBlueprintFunctionLibrary::GetCursorWorldRay(FVector& CameraLocation, FVector& RayOrigin, FVector& RayDirection)
{
	if (GCurrentLevelEditingViewportClient == nullptr)
	{
		return false;
	}

	FViewportCursorLocation CursorLocation = GCurrentLevelEditingViewportClient->GetCursorWorldLocationFromMousePos();

	CameraLocation = CursorLocation.GetOrigin();
	RayOrigin = CursorLocation.GetOrigin();
	RayDirection = CursorLocation.GetDirection();

	return true;
}

//Function that returns only the overlapping region between two world AABB extents
void ULandmassBlueprintFunctionLibrary::CombineWorldExtents(FVector4 ExtentsA, FVector4 ExtentsB, FVector4& CombinedExtents)
{
	CombinedExtents.X = FMath::Max(ExtentsA.X, ExtentsB.X);
	CombinedExtents.Y = FMath::Max(ExtentsA.Y, ExtentsB.Y);
	CombinedExtents.Z = FMath::Min(ExtentsA.Z, ExtentsB.Z);
	CombinedExtents.W = FMath::Min(ExtentsA.W, ExtentsB.W);
}

//Converts World Extents into Canvas coordinates with scaled UVs to match the partial draw area
void ULandmassBlueprintFunctionLibrary::WorldExtentsToCanvasCoordinates(FVector4 WorldExtents, FLandmassLandscapeInfo LandscapeInfo, FVector2D& ScreenPosition, FVector2D& ScreenSize, FVector2D& CoordinatePosition, FVector2D& CoordinateSize)
{
	FVector LandscapePosition = LandscapeInfo.LandscapeTransform.GetLocation();
	FVector LandscapeScale = LandscapeInfo.LandscapeTransform.GetScale3D();
	FVector2D RTWorldSize = FVector2D(LandscapeInfo.RenderTargetResolution) * FVector2D(LandscapeScale.X, LandscapeScale.Y);
	
	FVector2D ExtentsMin = FVector2D(WorldExtents.X, WorldExtents.Y) - FVector2D(LandscapePosition.X, LandscapePosition.Y);
	FVector2D ExtentsMax = FVector2D(WorldExtents.Z, WorldExtents.W) - FVector2D(LandscapePosition.X, LandscapePosition.Y);
	FVector2D ExtentSize = FVector2D(WorldExtents.Z, WorldExtents.W) - FVector2D(WorldExtents.X, WorldExtents.Y);

	ScreenPosition = ExtentsMin / RTWorldSize;
	ScreenPosition *= LandscapeInfo.RenderTargetResolution;
	ScreenPosition /= 4;
	ScreenPosition = FVector2D(FMath::FloorToFloat(ScreenPosition.X), FMath::FloorToFloat(ScreenPosition.Y));
	ScreenPosition *= 4;

	FVector2D ScreenPositionMax = ExtentsMax / RTWorldSize;
	ScreenPositionMax *= LandscapeInfo.RenderTargetResolution;
	ScreenPositionMax /= 4;
	ScreenPositionMax = FVector2D(FMath::CeilToFloat(ScreenPositionMax.X+1), FMath::CeilToFloat(ScreenPositionMax.Y+1));
	ScreenPositionMax *= 4;

	ScreenSize = ScreenPositionMax - ScreenPosition;

	CoordinatePosition = ScreenPosition / LandscapeInfo.RenderTargetResolution;
	CoordinateSize = ScreenSize / LandscapeInfo.RenderTargetResolution;

}

//Used when modifying texture settings via blueprint to force the texture to update since it may not 
void ULandmassBlueprintFunctionLibrary::ForceUpdateTexture(UTexture* InTexture)
{
	InTexture->Modify();
	InTexture->MarkPackageDirty();
	InTexture->PostEditChange();
	InTexture->UpdateResource();
}