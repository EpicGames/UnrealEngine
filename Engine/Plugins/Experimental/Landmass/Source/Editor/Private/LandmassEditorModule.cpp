// Copyright Epic Games, Inc. All Rights Reserved.

#include "LandmassEditorModule.h"
#include "Modules/ModuleManager.h"


class FLandmassEditorModule : public ILandmassEditorModuleInterface
{
public:
	virtual void StartupModule() override
	{
		// If a plugin's *content* directory contains Blutilities (blueprints inheriting classes from the Blutility engine module), there is no
		// guarantee that the Blutility engine module is loaded when the plugin gets loaded. The following is a workaround to enforce that the
		// required parent classes are loaded before we try to compile this plugin's content.
		FModuleManager::Get().LoadModuleChecked("Blutility");
	}

	virtual void ShutdownModule() override
	{
	}
};

//////////////////////////////////////////////////////////////////////////

IMPLEMENT_MODULE(FLandmassEditorModule, LandmassEditor);

