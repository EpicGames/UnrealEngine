// Copyright Epic Games, Inc. All Rights Reserved.

#include "LandmassErosionBrushBase.h"

#include "Modules/ModuleManager.h"
#include "EngineUtils.h"
#include "LevelEditor.h"
#include "Landscape.h"
#include "LandscapeEditTypes.h"
#include "LandscapeModule.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LandmassErosionBrushBase)

DEFINE_LOG_CATEGORY(LandmassErosionBrush);

ALandmassErosionBrushBase::ALandmassErosionBrushBase()
{
	if (!HasAnyFlags(EObjectFlags::RF_ClassDefaultObject))
	{
		FLevelEditorModule& LevelEditor = FModuleManager::GetModuleChecked<FLevelEditorModule>("LevelEditor");
		OnActorSelectionChangedHandle = LevelEditor.OnActorSelectionChanged().AddUObject(this, &ALandmassErosionBrushBase::HandleActorSelectionChanged);
	}

	SetCanAffectHeightmap(true);
}

void ALandmassErosionBrushBase::OnConstruction(const FTransform& Transform)
{
	Super::OnConstruction(Transform);
	FindAndAssignLandscape();
}

void ALandmassErosionBrushBase::SetTargetLandscape(ALandscape* InTargetLandscape)
{
#if WITH_EDITOR

	if (GetWorld()->WorldType != EWorldType::Editor)
	{
		return;
	}

	if (OwningLandscape != InTargetLandscape)
	{
		if (OwningLandscape)
		{
			OwningLandscape->RemoveBrush(this);
		}

		if (!InTargetLandscape)
		{
			if (OwningLandscape != nullptr)
			{
				// This can occur if the RemoveBrush call above did not do anything because the manager
				// was removed from the landscape in some other way (probably in landscape mode panel)
				SetOwningLandscape(nullptr);
			}
			return;
		}

		if (InTargetLandscape->IsTemplate())
		{
			UE_LOGF(LandmassErosionBrush, Warning, "Landscape target for Landmass Erosion Brush is a template. Unable to attach manager.");
			SetOwningLandscape(nullptr);
		}
	}
#endif
}

ALandscape* ALandmassErosionBrushBase::GetLandscape()
{
	return DetailPanelLandscape.Get();
}

void ALandmassErosionBrushBase::SetOwningLandscape(ALandscape* InOwningLandscape)
{
	Super::SetOwningLandscape(InOwningLandscape);

	DetailPanelLandscape = OwningLandscape;
}

void ALandmassErosionBrushBase::FindAndAssignLandscape()
{
	//Skip if this is a transient class
	if (this->HasAnyFlags(RF_Transient))
	{
		return;
	}

	for (TActorIterator<ALandscape> LandscapeIterator(GetWorld()); LandscapeIterator; ++LandscapeIterator)
	{
		if (!LandscapeIterator->IsTemplate())
		{
			SetTargetLandscape(*LandscapeIterator);
			break;
		}
	}
}

// We override PostEditChange to allow the users to change the owning landscape via a property displayed in the detail panel.
void ALandmassErosionBrushBase::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	// Do a bunch of checks to make sure that we don't try to do anything when the editing is happening inside the blueprint editor.
	UWorld* World = GetWorld();
	if (IsTemplate() || !IsValidChecked(this) || !IsValid(World) || World->WorldType != EWorldType::Editor)
	{
		return;
	}

	if (PropertyChangedEvent.Property
		&& (PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(ALandmassErosionBrushBase, DetailPanelLandscape)))
	{
		SetTargetLandscape(DetailPanelLandscape.Get());
	}
}

void ALandmassErosionBrushBase::HandleActorSelectionChanged(const TArray<UObject*>& NewSelection, bool bForceRefresh)
{
	if (!IsTemplate())
	{
		bool bUpdateActor = false;
		if (bWasSelected && !NewSelection.Contains(this))
		{
			bWasSelected = false;
			bUpdateActor = true;
		}
		if (!bWasSelected && NewSelection.Contains(this))
		{
			bWasSelected = true;
			bUpdateActor = true;
		}
		if (bUpdateActor)
		{
			ActorSelectionChanged(bWasSelected);
		}
	}
}

void ALandmassErosionBrushBase::ActorSelectionChanged_Implementation(bool bSelected)
{

}

