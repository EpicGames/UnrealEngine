// Copyright Epic Games, Inc. All Rights Reserved.

#include "LandmassManagerBase.h"

#include "Modules/ModuleManager.h"
#include "LevelEditor.h"
#include "Landscape.h"
#include "LandscapeEditTypes.h"
#include "LandscapeModule.h"
#include "Kismet/KismetMathLibrary.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LandmassManagerBase)

DEFINE_LOG_CATEGORY(LandmassManager);

ALandmassManagerBase::ALandmassManagerBase()
{
	BrushTreeDepth = 3;
	if (!HasAnyFlags(EObjectFlags::RF_ClassDefaultObject))
	{
		FLevelEditorModule& LevelEditor = FModuleManager::GetModuleChecked<FLevelEditorModule>("LevelEditor");
		OnActorSelectionChangedHandle = LevelEditor.OnActorSelectionChanged().AddUObject(this, &ALandmassManagerBase::HandleActorSelectionChanged);
	}

	// Explicitly set flag so this can be placed as a BP Brush from the landscape editor 
	SetCanAffectHeightmap(true);
}

void ALandmassManagerBase::SetTargetLandscape(ALandscape* InTargetLandscape)
{
#if WITH_EDITOR
	if (OwningLandscape != InTargetLandscape)
	{
		if (OwningLandscape)
		{
			OwningLandscape->RemoveBrush(this);
		}

		if (!InTargetLandscape)
		{
			if (OwningLandscape != nullptr)
			{
				// This can occur if the RemoveBrush call above did not do anything because the manager
				// was removed from the landscape in some other way (probably in landscape mode panel)
				SetOwningLandscape(nullptr);
			}
			return;
		}

		if (InTargetLandscape->IsTemplate())
		{
			UE_LOGF(LandmassManager, Warning, "Landscape target for Landmass Brush manager is a template. Unable to attach manager.");
			SetOwningLandscape(nullptr);
		}
	}
#endif
}

//Copied from KismetMathlibrary because the function was note exported via ENGINE_API
int32 ALandmassManagerBase::Convert2DTo1D(const FIntPoint& Index2D, int32 XSize)
{
	const bool bInvalidBounds =
		Index2D.X < 0 ||
		Index2D.X >= XSize ||
		Index2D.Y < 0
		;

	if (bInvalidBounds)
	{
		return 0;
	}

	if (XSize <= 0)
	{
		return 0;
	}

	return Index2D.X + (Index2D.Y * XSize);
}

void ALandmassManagerBase::PopulateNodeTree()
{
	BrushNodeData.Empty();
	int32 NodeIndexOffset = 0;
	int32 ParentIndexOffset = 0;

	BrushTreeDepth = FMath::Clamp(BrushTreeDepth, 0, 4);

	FVector2D WorldSize = { LandscapeInformation.LandscapeTransform.GetScale3D().X, LandscapeInformation.LandscapeTransform.GetScale3D().Y };
	WorldSize *= LandscapeInformation.LandscapeQuads;

	//For each level
	for (int i = 0; i <= BrushTreeDepth; i++)
	{
		int32 SizeOfLevel = FMath::TruncToInt(FMath::Pow(2.0, i));
		int32 NodesInLevel = SizeOfLevel * SizeOfLevel;
		int32 SizeOfNextLevel = FMath::TruncToInt(FMath::Pow(2.0, i + 1));
		NodeIndexOffset += NodesInLevel;

		//Create empty nodes
		for (int j = 0; j <= NodesInLevel - 1; j++)
		{
			FVector2D NodeWorldSize = WorldSize / SizeOfLevel;

			FIntPoint NodeIndex2D = FIntPoint( j % SizeOfLevel, j / SizeOfLevel );
			FVector LandscapePosition = LandscapeInformation.LandscapeTransform.GetLocation();
			FVector2D NodeMinExtent = FVector2D(NodeIndex2D.X, NodeIndex2D.Y) * NodeWorldSize + FVector2D(LandscapePosition.X, LandscapePosition.Y);
			FVector4 CurrentNodeExtents = FVector4(NodeMinExtent.X, NodeMinExtent.Y, NodeMinExtent.X + NodeWorldSize.X, NodeMinExtent.Y + NodeWorldSize.Y);

			FBrushDataTree NewNode;
			NewNode.CurrentLevel = i;
			//Calculate the index of this node's parent by going up one level of subdivision
			NewNode.ParentIndex = Convert2DTo1D((NodeIndex2D / FIntPoint(2, 2)), SizeOfLevel / 2) + ParentIndexOffset;
			NewNode.Index_x0y0 = -1;
			NewNode.Index_x1y0 = -1;
			NewNode.Index_x0y1 = -1;
			NewNode.Index_x1y1 = -1;
			NewNode.ChildDataCount = 0;
			NewNode.NodeExtents = CurrentNodeExtents;

			if (i < BrushTreeDepth)
			{
				//Nodes contain the index of child nodes, calculate the index based on index from subdivision of level
				NewNode.Index_x0y0 = Convert2DTo1D( (NodeIndex2D * FIntPoint(2,2)) + FIntPoint(0,0), SizeOfNextLevel) +  NodeIndexOffset;
				NewNode.Index_x1y0 = Convert2DTo1D( (NodeIndex2D * FIntPoint(2,2)) + FIntPoint(1,0), SizeOfNextLevel) +  NodeIndexOffset;
				NewNode.Index_x0y1 = Convert2DTo1D( (NodeIndex2D * FIntPoint(2,2)) + FIntPoint(0,1), SizeOfNextLevel) +  NodeIndexOffset;
				NewNode.Index_x1y1 = Convert2DTo1D( (NodeIndex2D * FIntPoint(2,2)) + FIntPoint(1,1), SizeOfNextLevel) +  NodeIndexOffset;
			}

			BrushNodeData.Add(NewNode);

		}
		int32 NodesInPreviousLevel = FMath::TruncToInt(FMath::Pow(2.0, i - 1)) * FMath::TruncToInt(FMath::Pow(2.0, i - 1));
		ParentIndexOffset += NodesInPreviousLevel;

	}
}

TArray<ALandmassActor*> ALandmassManagerBase::GetActorsWithinModifiedNodes(TArray<int32>& InModifiedNodes)
{
	TArray<ALandmassActor*> ActorsInNodes;
	TArray<int32> NodesToCheck = InModifiedNodes;
	TArray<int32> ParentsToCheck = InModifiedNodes;
	TArray<int32> NodesToCheckNext;
	TArray<int32> ParentsToCheckNext;

	//Check each level of tree
	for (int i = 0; i <= BrushTreeDepth; i++)
	{
		NodesToCheckNext.Empty();

		//Check all nodes in the current level
		for (int32 CurrentNodeIndex : NodesToCheck)
		{
			if (!BrushNodeData[CurrentNodeIndex].BrushActors.IsEmpty())
			{
				//Get any brushes from those nodes
				for (ALandmassActor* CurrentActor : BrushNodeData[CurrentNodeIndex].BrushActors)
				{
					ActorsInNodes.AddUnique(CurrentActor);
				}
			}

			//If any of the node's children have data, add them to list to check at the next level
			if (i < BrushTreeDepth && BrushNodeData[CurrentNodeIndex].ChildDataCount > 0)
			{
				NodesToCheckNext.Add(BrushNodeData[CurrentNodeIndex].Index_x0y0);
				NodesToCheckNext.Add(BrushNodeData[CurrentNodeIndex].Index_x1y0);
				NodesToCheckNext.Add(BrushNodeData[CurrentNodeIndex].Index_x0y1);
				NodesToCheckNext.Add(BrushNodeData[CurrentNodeIndex].Index_x1y1);
			}
		}
		//Set the next level iteration to check those nodes that were found with data
		NodesToCheck = NodesToCheckNext;
		ParentsToCheckNext.Empty();

		//Also check the node's parent for data, since it is possible the search began with leaf nodes
		for (int32 CurrentNodeIndex : ParentsToCheck)
		{
			FBrushDataTree CheckNode = BrushNodeData[CurrentNodeIndex];
			int32 CurrentParentIndex = CheckNode.ParentIndex;

			if (CurrentNodeIndex > 0 && CheckNode.CurrentLevel > 0)
			{
				if (!BrushNodeData[CurrentParentIndex].BrushActors.IsEmpty())
				{
					//Get all brushes from the parent node
					for (ALandmassActor* CurrentActor : BrushNodeData[CurrentParentIndex].BrushActors)
					{
						ActorsInNodes.AddUnique(CurrentActor);
					}
				}

				ParentsToCheckNext.AddUnique(CurrentParentIndex);
			}
		}
		//Set the next level's iteration to check the parents of checked nodes
		ParentsToCheck = ParentsToCheckNext;

	}

	return ActorsInNodes;
}

//Go through nodes and update nodes to include the number of brushes in all their children under ChildDataCount
void ALandmassManagerBase::UpdateChildDataCounts()
{
	int32 CurrentActorCount;

	for (FBrushDataTree& CurrentNode : BrushNodeData)
	{
		CurrentNode.ChildDataCount = 0;
		CurrentActorCount = CurrentNode.BrushActors.Num();

		if (CurrentActorCount > 0 && CurrentNode.CurrentLevel > 0)
		{
			int32 CurrentLevel = CurrentNode.CurrentLevel;
			int32 ParentToUpdate = CurrentNode.ParentIndex;

			for (int i = 0; i <= CurrentLevel - 1; i++)
			{
				FBrushDataTree& CurrentParentData = BrushNodeData[ParentToUpdate];
				BrushNodeData[ParentToUpdate].ChildDataCount += CurrentActorCount;
				ParentToUpdate = CurrentParentData.ParentIndex;
			}
		}

	}
}

//When a list of nodes includes all the child nodes of a parent, replace the children with the parent
void ALandmassManagerBase::ConsolidateNodes(TArray<int32>& NodesToConsolidate)
{
	TArray<int32> FullyOccupiedParentNodes;

	//Check each tree level
	for (int i = 0; i <= BrushTreeDepth - 1; i++)
	{
		FullyOccupiedParentNodes.Empty();

		for (int32 CurrentCheckIndex : NodesToConsolidate)
		{
			//Check if Node list contains all the children of a given node
			int32 CurrentParentIndex = BrushNodeData[CurrentCheckIndex].ParentIndex;
			FBrushDataTree CurrentParentData = BrushNodeData[CurrentParentIndex];
			int32 Child_x0y0 = NodesToConsolidate.Find(CurrentParentData.Index_x0y0);
			int32 Child_x1y0 = NodesToConsolidate.Find(CurrentParentData.Index_x1y0);
			int32 Child_x0y1 = NodesToConsolidate.Find(CurrentParentData.Index_x0y1);
			int32 Child_x1y1 = NodesToConsolidate.Find(CurrentParentData.Index_x1y1);

			//If it does, mark it as a fully occupied parent node
			if (Child_x0y0 >= 0 && Child_x1y0 >= 0 && Child_x0y1 >= 0 && Child_x1y1 >= 0)
			{
				FullyOccupiedParentNodes.AddUnique(CurrentParentIndex);
			}
		}

		//Since we added the parent node to the list and all its children were also occupied, remove the children
		for (int32 FullyOccupiedNode : FullyOccupiedParentNodes)
		{
			FBrushDataTree CurrentNodeData = BrushNodeData[FullyOccupiedNode];
			NodesToConsolidate.Remove(CurrentNodeData.Index_x0y0);
			NodesToConsolidate.Remove(CurrentNodeData.Index_x1y0);
			NodesToConsolidate.Remove(CurrentNodeData.Index_x0y1);
			NodesToConsolidate.Remove(CurrentNodeData.Index_x1y1);
			NodesToConsolidate.Insert(FullyOccupiedNode, 0);
		}
	}
}

TArray<int32> ALandmassManagerBase::GetNodesWithinExtents(FVector4& InExtents)
{
	TArray<int32> OverlappedNodes = { 0 };
	TArray<int32> NodesToCheckNext;
	FVector2D CheckExtentMin = { InExtents.X, InExtents.Y };
	FVector2D CheckExtentMax = { InExtents.Z, InExtents.W };

	//Check each level of tree
	for (int i = 0; i <= BrushTreeDepth - 1; i++)
	{
		NodesToCheckNext.Empty();

		//Check which nodes the given extents overlap, down to the smallest level
		for (int32 CurrentCheckIndex : OverlappedNodes)
		{
			FVector4 CurrentNodeExtents = BrushNodeData[CurrentCheckIndex].NodeExtents;
			FVector2D CurrentNodeCenter = (FVector2D(CurrentNodeExtents.X, CurrentNodeExtents.Y) + FVector2D(CurrentNodeExtents.Z, CurrentNodeExtents.W)) / 2;
			if ((CheckExtentMin.X < CurrentNodeCenter.X) && (CheckExtentMin.Y < CurrentNodeCenter.Y))
			{
				NodesToCheckNext.Add(BrushNodeData[CurrentCheckIndex].Index_x0y0);
			}
			if ((CheckExtentMax.X > CurrentNodeCenter.X) && (CheckExtentMin.Y < CurrentNodeCenter.Y))
			{
				NodesToCheckNext.Add(BrushNodeData[CurrentCheckIndex].Index_x1y0);
			}
			if ((CheckExtentMin.X < CurrentNodeCenter.X) && (CheckExtentMax.Y > CurrentNodeCenter.Y))
			{
				NodesToCheckNext.Add(BrushNodeData[CurrentCheckIndex].Index_x0y1);
			}
			if ((CheckExtentMax.X > CurrentNodeCenter.X) && (CheckExtentMax.Y > CurrentNodeCenter.Y))
			{
				NodesToCheckNext.Add(BrushNodeData[CurrentCheckIndex].Index_x1y1);
			}
		}
		OverlappedNodes = NodesToCheckNext;

	}
	//Consolidate the nodes back up to the lowest level possible. ie, if all nodes are returned, just use the root node
	ConsolidateNodes(OverlappedNodes);
	return OverlappedNodes;
}

//Go through and remove the Brush from the entire tree
TArray<int32> ALandmassManagerBase::RemoveBrushFromTree(ALandmassActor* BrushToRemove)
{
	TArray<int32> RemovedIndices;
	TArray<int32> NodesToCheck = {0};
	TArray<int32> NodesToCheckNext;

	for (int i = 0; i <= BrushTreeDepth; i++)
	{
		NodesToCheckNext.Empty();

		for (int32 CurrentCheckIndex : NodesToCheck)
		{
			if (!BrushNodeData[CurrentCheckIndex].BrushActors.IsEmpty())
			{
				int32 RemovalIndex = BrushNodeData[CurrentCheckIndex].BrushActors.Find(BrushToRemove);
				if (RemovalIndex >= 0)
				{
					BrushNodeData[CurrentCheckIndex].BrushActors.Remove(BrushToRemove);
					RemovedIndices.Add(CurrentCheckIndex);
				}
				//Remove all null entries while at it
				BrushNodeData[CurrentCheckIndex].BrushActors.Remove(nullptr);
			}

			if (BrushNodeData[CurrentCheckIndex].ChildDataCount > 0)
			{
				NodesToCheckNext.Add(BrushNodeData[CurrentCheckIndex].Index_x0y0);
				NodesToCheckNext.Add(BrushNodeData[CurrentCheckIndex].Index_x1y0);
				NodesToCheckNext.Add(BrushNodeData[CurrentCheckIndex].Index_x0y1);
				NodesToCheckNext.Add(BrushNodeData[CurrentCheckIndex].Index_x1y1);
			}

		}

		NodesToCheck = NodesToCheckNext;
	}

	return RemovedIndices;
}

//Sort one array to match another. Used to ensure the list of modified brushes pulled from the tree matches the order from the complete brush list to maintain draw order
TArray<ALandmassActor*> ALandmassManagerBase::SortBrushes(TArray<ALandmassActor*> BrushArrayToMatch, TArray<ALandmassActor*> ActorsToSort)
{

	TMap<ALandmassActor*, int32> MapToSort;

	for (ALandmassActor* CurrentBrush : ActorsToSort)
	{
		int32 IndexInSourceArray = BrushArrayToMatch.Find(CurrentBrush);
		if(IndexInSourceArray >= 0)
		{
			MapToSort.Add(CurrentBrush, IndexInSourceArray);
		}
		
	}

	//Algo::SortBy(SortedKeys, [&SortedKeys](const AActor* A) { return  SortedKeys[A]; }, TGreater<>{});

	MapToSort.ValueSort([](int32 A, int32 B) { return A < B; });
	TArray<ALandmassActor*> SortedActors;
	MapToSort.GenerateKeyArray(SortedActors);

	return SortedActors;

}

void ALandmassManagerBase::AddBrushToTree(ALandmassActor* BrushToAdd, FVector4 InExtents, bool InMapToWholeLandscape, FVector4& ModifiedExtents, TArray<ALandmassActor*>& InvalidatedBrushes, TArray<int32>& ModifiedNodes)
{

	if (BrushNodeData.IsEmpty())
	{
		UE_LOGF(LandmassManager, Warning, "Landmass Manager Brush Tree was Empty but a Brush attempted to add itself to the tree.");
		return;
	}

	TArray<int32> RemovedFromNodes = RemoveBrushFromTree(BrushToAdd);
	ModifiedNodes.Empty();

	if (!InMapToWholeLandscape)
	{
		//Get the Nodes within the modified Brush Extents
		ModifiedNodes = GetNodesWithinExtents(InExtents);
	}
	else
	{
		//If Brush affects Whole Landscape, modify the Root Node
		ModifiedNodes = { 0 };
	}

	for (int32 ModifiedNode : ModifiedNodes)
	{
		BrushNodeData[ModifiedNode].BrushActors.AddUnique(BrushToAdd);
	}
	
	//Update the child data counts each node contains for its children
	UpdateChildDataCounts();

	//Invalidate the currently modified Nodes and Nodes Brush was previously in
	for (int32 RemovedNodeIndex : RemovedFromNodes)
	{
		ModifiedNodes.AddUnique(RemovedNodeIndex);
	}

	ModifiedExtents = FVector4(TNumericLimits<double>::Max(), TNumericLimits<double>::Max(), TNumericLimits<double>::Min(), TNumericLimits<double>::Min());
	for (int32 ModifiedNodeIndex : ModifiedNodes)
	{
		FVector4 CurrentExtents = BrushNodeData[ModifiedNodeIndex].NodeExtents;

		//Min of Extent Mins
		ModifiedExtents.X = FMath::Min(CurrentExtents.X, ModifiedExtents.X);
		ModifiedExtents.Y = FMath::Min(CurrentExtents.Y, ModifiedExtents.Y);
		//Max of Extents Maxs
		ModifiedExtents.Z = FMath::Max(CurrentExtents.Z, ModifiedExtents.Z);
		ModifiedExtents.W = FMath::Max(CurrentExtents.W, ModifiedExtents.W);
	}

	//Now re-gather ModfiedNodes using the combined Modified Extents including nodes Actor removed from
	ModifiedNodes = GetNodesWithinExtents(ModifiedExtents);

	InvalidatedBrushes = GetActorsWithinModifiedNodes(ModifiedNodes);
}

void ALandmassManagerBase::AddBrushToArray(ALandmassActor* BrushToAdd)
{
	LandmassBrushes.AddUnique(BrushToAdd);
	int32 LastBrushIndex = LandmassBrushes.Num() - 1;
	for (int i = 0; i <= LastBrushIndex; i++)
	{
		//Remove null blueprints caused by deleting brushes
		if (LandmassBrushes[LastBrushIndex - i] == nullptr)
		{
			LandmassBrushes.RemoveAt(LastBrushIndex - i);
		}
		//Remove REINST blueprints caused by recompiling blueprint
		else
		{
			FString BrushName = LandmassBrushes[LastBrushIndex-i]->GetName();
			if (BrushName.StartsWith("REINST"))
			{
				LandmassBrushes.RemoveAt(LastBrushIndex - i);
			}
		}
	}
}

void ALandmassManagerBase::RequestUpdateFromBrush_Implementation(ALandmassActor* BrushRequestingUpdate)
{

}

void ALandmassManagerBase::DrawBrushMaterial_Implementation(ALandmassActor* Brush, UMaterialInterface* BrushMaterial)
{

}

void ALandmassManagerBase::LaunchLandmassEditor_Implementation(ALandmassActor* BrushRequestingEditor)
{

}

void ALandmassManagerBase::TogglePreviewMode_Implementation(bool bEnablePreviewMode)
{

}

void ALandmassManagerBase::SetCaptureBoundaryNormals(bool)
{
}

void ALandmassManagerBase::MoveBrushUp(ALandmassActor* BrushToMove)
{
	int32 BrushIndex = LandmassBrushes.Find(BrushToMove);
	if (BrushIndex >= 0 && BrushIndex < (LandmassBrushes.Num() - 1))
	{
		LandmassBrushes.Swap(BrushIndex, BrushIndex + 1);
	}

	RequestUpdateFromBrush(BrushToMove);
}

void ALandmassManagerBase::MoveBrushDown(ALandmassActor* BrushToMove)
{
	int32 BrushIndex = LandmassBrushes.Find(BrushToMove);
	if (BrushIndex > 0)
	{
		LandmassBrushes.Swap(BrushIndex, BrushIndex - 1);
	}

	RequestUpdateFromBrush(BrushToMove);
}

void ALandmassManagerBase::MoveBrushToTop(ALandmassActor* BrushToMove)
{
	int32 BrushIndex = LandmassBrushes.Find(BrushToMove);
	if (BrushIndex >= 0 && BrushIndex < (LandmassBrushes.Num() - 1))
	{
		LandmassBrushes.Remove(BrushToMove);
		LandmassBrushes.Add(BrushToMove);
	}

	RequestUpdateFromBrush(BrushToMove);
}

void ALandmassManagerBase::MoveBrushToBottom(ALandmassActor* BrushToMove)
{
	int32 BrushIndex = LandmassBrushes.Find(BrushToMove);
	if (BrushIndex > 0)
	{
		LandmassBrushes.Remove(BrushToMove);
		LandmassBrushes.Insert(BrushToMove, 0);
	}

	RequestUpdateFromBrush(BrushToMove);
}

ALandscape* ALandmassManagerBase::GetLandscape()
{
	return DetailPanelLandscape.Get();
}

void ALandmassManagerBase::SetOwningLandscape(ALandscape* InOwningLandscape)
{
	Super::SetOwningLandscape(InOwningLandscape);

	DetailPanelLandscape = OwningLandscape;
}

// We override PostEditChange to allow the users to change the owning landscape via a property displayed in the detail panel.
void ALandmassManagerBase::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	// Do a bunch of checks to make sure that we don't try to do anything when the editing is happening inside the blueprint editor.
	UWorld* World = GetWorld();
	if (IsTemplate() || !IsValidChecked(this) || !IsValid(World) || World->WorldType != EWorldType::Editor)
	{
		return;
	}

	if (PropertyChangedEvent.Property
		&& (PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(ALandmassManagerBase, DetailPanelLandscape)))
	{
		SetTargetLandscape(DetailPanelLandscape.Get());
	}
}

void ALandmassManagerBase::HandleActorSelectionChanged(const TArray<UObject*>& NewSelection, bool bForceRefresh)
{
	if (!IsTemplate())
	{
		bool bUpdateActor = false;
		if (bWasSelected && !NewSelection.Contains(this))
		{
			bWasSelected = false;
			bUpdateActor = true;
		}
		if (!bWasSelected && NewSelection.Contains(this))
		{
			bWasSelected = true;
			bUpdateActor = true;
		}
		if (bUpdateActor)
		{
			ActorSelectionChanged(bWasSelected);
		}
	}
}

void ALandmassManagerBase::ActorSelectionChanged_Implementation(bool bSelected)
{

}