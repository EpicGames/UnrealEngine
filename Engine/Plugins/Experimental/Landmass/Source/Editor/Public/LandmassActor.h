// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GameFramework/Actor.h"
#include "LandscapeBlueprintBrushBase.h"
#include "LandmassActor.generated.h"


class ALandmassManagerBase;
class UStaticMeshComponent;
class UMaterialInterface;
class UMaterialInstanceDynamic;

DECLARE_DYNAMIC_MULTICAST_DELEGATE(FOnBrushUpdatedDelegate);

UENUM(BlueprintType)
enum class EBrushBlendMode : uint8
{
	AlphaBlend,
	Min,
	Max,
	Additive
};

UCLASS(Blueprintable, PrioritizeCategories = ("PreviewMode", "LayerOrdering", "BrushSettings", "HeightmapSettings", "WeightmapSettings", "Debug"),
	hidecategories = (Replication, Input, LOD, Actor, Cooking, Rendering, Collision, HLOD, Physics, Networking, LevelInstance, DataLayers))
class ALandmassActor : public AActor
{
	GENERATED_BODY()

	virtual void OnConstruction(const FTransform& Transform) override;

public:
	ALandmassActor();

	UFUNCTION(BlueprintNativeEvent, CallInEditor, BlueprintCallable, Category = "Tick")
	void CustomTick(float DeltaSeconds);

	virtual bool IsEditorOnly() const override { return true; }

	virtual bool ShouldTickIfViewportsOnly() const override;
	virtual void Tick(float DeltaSeconds) override;

	UFUNCTION(BlueprintCallable, CallInEditor, Category = Default)
	void RenderLayer_Native(const FLandscapeBrushParameters& InParameters);

	UFUNCTION(BlueprintNativeEvent, CallInEditor, meta = (ForceAsFunction) , Category = Default)
	void RenderLayer(const FLandscapeBrushParameters& InParameters);

	UFUNCTION(BlueprintCallable, CallInEditor, Category = "PreviewMode", meta = (DisplayPriority = 1))
	void FastPreviewMode();

	UFUNCTION(BlueprintCallable, CallInEditor, Category = "PreviewMode", meta = (DisplayPriority = 2))
	void RestoreLandscapeEditing();

	UFUNCTION(BlueprintCallable, CallInEditor, Category = "LayerOrdering", meta = (DisplayPriority = 3))
	void MoveBrushUp();

	UFUNCTION(BlueprintCallable, CallInEditor, Category = "LayerOrdering", meta = (DisplayPriority = 4))
	void MoveBrushDown();

	UFUNCTION(BlueprintCallable, CallInEditor, Category = "LayerOrdering", meta = (DisplayPriority = 5))
	void MoveToTop();

	UFUNCTION(BlueprintCallable, CallInEditor, Category = "LayerOrdering", meta = (DisplayPriority = 6))
	void MoveToBottom();
	
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "BrushSettings", meta = (DisplayPriority = 8))
	float BrushSize = 4096;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "BrushSettings", meta = (DisplayPriority = 9))
	bool DrawToEntireLandscape = false;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "BrushSettings", meta = (DisplayPriority = 10))
	bool AffectsHeightmap = true;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "BrushSettings", meta = (DisplayPriority = 11))
	bool AffectsWeightmaps = true;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "BrushSettings", meta = (DisplayPriority = 12))
	bool AffectsVisibility = false;
	
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "HeightmapSettings", meta = (DisplayPriority = 13))
	EBrushBlendMode HeightBlendMode = EBrushBlendMode::AlphaBlend;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "HeightmapSettings", meta = (DisplayPriority = 14))
	TObjectPtr<UMaterialInterface> HeightMaterial;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "WeightmapSettings", meta = (DisplayPriority = 15))
	EBrushBlendMode WeightMapBlendMode = EBrushBlendMode::AlphaBlend;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "WeightmapSettings", meta = (DisplayPriority = 16))
	TObjectPtr<UMaterialInterface> WeightmapMaterial;

	UPROPERTY(BlueprintReadOnly, Category = "Debug", meta = (DisplayPriority = 17))
	FVector4 BrushExtents;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Debug", meta = (DisplayPriority = 18))
	TArray<FName> WeightmapLayers;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Debug", meta = (DisplayPriority = 19))
	FLandscapeBrushParameters BrushRenderParameters;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Debug", meta = (DisplayPriority = 20))
	TObjectPtr<ALandmassManagerBase> BrushManager;

	UFUNCTION(BlueprintCallable, category = "Default")
	void SetEditorTickEnabled(bool bEnabled) { EditorTickIsEnabled = bEnabled; }

	UFUNCTION(BlueprintCallable, Category = "Landmass")
	void UpdateBrushExtents();

	UPROPERTY()
	bool EditorTickIsEnabled = false;

	UFUNCTION(BlueprintNativeEvent, CallInEditor, BlueprintCallable, Category = "Selection")
	void ActorSelectionChanged(bool bSelected);

	void FindOrSpawnManager();

	UFUNCTION(BlueprintCallable, CallInEditor, Category = Default)
	void DrawBrushMaterial(UMaterialInterface* Material);

	UFUNCTION(BlueprintCallable, CallInEditor, Category = Default)
	void SetMeshExentsMaterial(UMaterialInterface* Material);

	UPROPERTY(BlueprintAssignable, Category = "Landmass")
	FOnBrushUpdatedDelegate OnBrushUpdated;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Transient, Category = "Debug", meta = (DisplayPriority = 21))
	TObjectPtr<UMaterialInstanceDynamic> ExtentsPreviewMID;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Transient, Category = "Debug", meta = (DisplayPriority = 22))
	TObjectPtr<UMaterialInstanceDynamic> HeightmapRenderMID;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Transient, Category = "Debug", meta = (DisplayPriority = 23))
	TObjectPtr<UMaterialInstanceDynamic> WeightmapRenderMID;

private:
	bool bWasSelected = false;

	UPROPERTY()
	TObjectPtr<UStaticMeshComponent> MeshExtentsQuad;

	UPROPERTY()
	TObjectPtr<UStaticMeshComponent> BrushSpriteMesh;

	FDelegateHandle OnActorSelectionChangedHandle;

	/** Called when the editor selection has changed. */
	void HandleActorSelectionChanged(const TArray<UObject*>& NewSelection, bool bForceRefresh);

};
