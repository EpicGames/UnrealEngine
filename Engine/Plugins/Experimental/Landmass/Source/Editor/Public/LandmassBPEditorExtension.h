// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Kismet/BlueprintFunctionLibrary.h"
#include "LandmassManagerBase.h"
#include "LandmassBPEditorExtension.generated.h"


UCLASS(meta = (Namespace = "Landmass"))
class ULandmassBlueprintFunctionLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:

	UFUNCTION(BlueprintCallable, meta = (DisplayName = "Get Cursor World Ray", Keywords = "Get Cursor World Ray", UnsafeDuringActorConstruction = "true"), Category = Rendering)
	static bool GetCursorWorldRay(FVector& CameraLocation, FVector& RayOrigin, FVector& RayDirection);

	UFUNCTION(BlueprintCallable, BlueprintPure, meta = (DisplayName = "Get Overlapping World Extents", Keywords = "Combine World Extents", UnsafeDuringActorConstruction = "false"), Category = Landmass)
	static void CombineWorldExtents(FVector4 ExtentsA, FVector4 ExtentsB, FVector4& CombinedExtents);

	UFUNCTION(BlueprintCallable, BlueprintPure, meta = (DisplayName = "World Extents to Landmass Coordinates", Keywords = "World Extents to Landmass Coordinates", UnsafeDuringActorConstruction = "false"), Category = Landmass)
	static void WorldExtentsToCanvasCoordinates(FVector4 WorldExtents, FLandmassLandscapeInfo LandscapeInfo, FVector2D& ScreenPosition, FVector2D& ScreenSize, FVector2D& CoordinatePosition, FVector2D& CoordinateSize);

	UFUNCTION(BlueprintCallable, Category = Rendering)
	static void ForceUpdateTexture(UTexture* InTexture);
};
