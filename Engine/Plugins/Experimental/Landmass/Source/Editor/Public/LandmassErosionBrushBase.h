// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "LandscapeBlueprintBrushBase.h"
#include "LandscapeEditTypes.h"
#include "Logging/LogMacros.h"
#include "LandmassErosionBrushBase.generated.h"

LANDMASSEDITOR_API DECLARE_LOG_CATEGORY_EXTERN(LandmassErosionBrush, Display, All);

UCLASS(Blueprintable, hidecategories = (Replication, Input, LOD, Actor, Cooking, Rendering))
class ALandmassErosionBrushBase : public ALandscapeBlueprintBrushBase
{
	GENERATED_BODY()

	virtual void OnConstruction(const FTransform& Transform) override;

public:

	ALandmassErosionBrushBase();

	virtual bool IsEditorOnly() const override { return true; }

	// Adds the brush to the given landscape, removing it from any previous one. This differs from SetOwningLandscape
	// in that SetOwningLandscape is called by the landscape itself from AddBrushToLayer to update the manager.
	UFUNCTION(BlueprintCallable, category = "Landmass")
	virtual void SetTargetLandscape(ALandscape* InOwningLandscape);

	UFUNCTION(BlueprintCallable, category = "Landmass")
	ALandscape* GetLandscape();

	UFUNCTION(BlueprintNativeEvent, CallInEditor, BlueprintCallable, Category = "Selection")
	void ActorSelectionChanged(bool bSelected);

	void FindAndAssignLandscape();

#if WITH_EDITOR
	// ALandscapeBlueprintBrushBase
	virtual void SetOwningLandscape(class ALandscape* InOwningLandscape) override;
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif

private:
	bool bWasSelected = false;

	FDelegateHandle OnActorSelectionChangedHandle;

	/** Called when the editor selection has changed. */
	void HandleActorSelectionChanged(const TArray<UObject*>& NewSelection, bool bForceRefresh);

#if WITH_EDITORONLY_DATA
	//~ This is transient because SetOwningLandscape is called in ALandscape::PostLoad.
	/**
	 * The owning landscape.
	 */
	UPROPERTY(EditAnywhere, Category = Landscape, Transient, meta = (DisplayName = "Landscape"))
	TObjectPtr<ALandscape> DetailPanelLandscape;
#endif

};
