// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "LandscapeBlueprintBrushBase.h"
#include "LandmassActor.h"
#include "LandscapeEditTypes.h"
#include "Logging/LogMacros.h"
#include "LandmassManagerBase.generated.h"

LANDMASSEDITOR_API DECLARE_LOG_CATEGORY_EXTERN(LandmassManager, Display, All);

USTRUCT(BlueprintType)
struct FBrushDataTree
{
	GENERATED_BODY()
	FBrushDataTree() {
		CurrentLevel = -1; ParentIndex = -1; Index_x0y0 = -1; Index_x1y0 = -1; Index_x0y1 = -1; Index_x1y1 = -1; ChildDataCount = 0; NodeExtents = FVector4(0, 0, 0, 0);
	}

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Landmass)
	int32 CurrentLevel;
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Landmass)
	int32 ParentIndex;
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Landmass)
	TArray<TObjectPtr<ALandmassActor>> BrushActors;
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Landmass)
	int32 Index_x0y0;
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Landmass)
	int32 Index_x1y0;
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Landmass)
	int32 Index_x0y1;
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Landmass)
	int32 Index_x1y1;
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Landmass)
	int32 ChildDataCount;
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Landmass)
	FVector4 NodeExtents;
};

USTRUCT(BlueprintType)
struct FLandmassLandscapeInfo
{
	GENERATED_BODY()
	FLandmassLandscapeInfo() {
		LandscapeTransform = FTransform(); LandscapeQuads = { 0,0 }; RenderTargetResolution = { 0,0 };
	}

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Landmass)
	FTransform LandscapeTransform;
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Landmass)
	FIntPoint LandscapeQuads;
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Landmass)
	FIntPoint RenderTargetResolution;

};

UCLASS(Blueprintable, hidecategories = (Replication, Input, LOD, Actor, Cooking, Rendering))
class ALandmassManagerBase : public ALandscapeBlueprintBrushBase
{
	GENERATED_BODY()

public:

	ALandmassManagerBase();

	virtual bool IsEditorOnly() const override { return true; }

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Quadtree)
	TArray<FBrushDataTree> BrushNodeData;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Landmass)
	FLandmassLandscapeInfo LandscapeInformation;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Quadtree)
	int32 BrushTreeDepth;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Landmass)
	TArray<TObjectPtr<ALandmassActor>> LandmassBrushes;

	// Adds the brush to the given landscape, removing it from any previous one. This differs from SetOwningLandscape
	// in that SetOwningLandscape is called by the landscape itself from AddBrushToLayer to update the manager.
	UFUNCTION(BlueprintCallable, category = "Landmass")
	virtual void SetTargetLandscape(ALandscape* InOwningLandscape);

	int32 Convert2DTo1D(const FIntPoint& Index2D, int32 XSize);

	UFUNCTION(BlueprintCallable, category = "Landmass")
	void PopulateNodeTree();

	UFUNCTION(BlueprintCallable, category = "Landmass")
	TArray<ALandmassActor*> GetActorsWithinModifiedNodes(TArray<int32>& InModifiedNodes);

	UFUNCTION(BlueprintCallable, category = "Landmass")
	void UpdateChildDataCounts();

	UFUNCTION(BlueprintCallable, category = "Landmass")
	void ConsolidateNodes(TArray<int32>& NodesToConsolidate);

	UFUNCTION(BlueprintCallable, category = "Landmass")
	TArray<int32> GetNodesWithinExtents(FVector4& InExtents);

	UFUNCTION(BlueprintCallable, category = "Landmass")
	TArray<int32> RemoveBrushFromTree(ALandmassActor* BrushToRemove);

	UFUNCTION(BlueprintCallable, category = "Landmass")
	TArray<ALandmassActor*> SortBrushes(TArray<ALandmassActor*> BrushArrayToMatch, TArray<ALandmassActor*> ActorsToSort);

	UFUNCTION(BlueprintCallable, category = "Landmass")
	void AddBrushToTree(ALandmassActor* BrushToAdd, FVector4 InExtents, bool InMapToWholeLandscape, FVector4& ModifiedExtents, TArray<ALandmassActor*>& InvalidatedBrushes, TArray<int32>& ModifiedNodes);

	UFUNCTION(BlueprintCallable, category = "Landmass")
	void AddBrushToArray(ALandmassActor* BrushToAdd);

	UFUNCTION(BlueprintNativeEvent, BlueprintCallable, meta = (ForceAsFunction), Category = Default)
	void RequestUpdateFromBrush(ALandmassActor* BrushRequestingUpdate);

	UFUNCTION(BlueprintNativeEvent, BlueprintCallable, CallInEditor, meta = (ForceAsFunction, DefaultToSelf = "Brush"), Category = Default)
	void DrawBrushMaterial(ALandmassActor* Brush, UMaterialInterface* BrushMaterial);

	UFUNCTION(BlueprintNativeEvent, BlueprintCallable, meta = (ForceAsFunction), Category = Default)
	void LaunchLandmassEditor(ALandmassActor* BrushRequestingEditor);

	UFUNCTION(BlueprintNativeEvent, BlueprintCallable, meta = (ForceAsFunction), Category = Default)
	void TogglePreviewMode(bool bEnablePreviewMode);

	UE_DEPRECATED(5.6, "This no longer has an effect")
	UFUNCTION(BlueprintCallable, Category = Default)
	void SetCaptureBoundaryNormals(bool bInCaptureBoundaryNormals);
	
	void MoveBrushUp(ALandmassActor* BrushToMove);
	void MoveBrushDown(ALandmassActor* BrushToMove);
	void MoveBrushToTop(ALandmassActor* BrushToMove);
	void MoveBrushToBottom(ALandmassActor* BrushToMove);

	UFUNCTION(BlueprintCallable, category = "Landmass")
	ALandscape* GetLandscape();

	UFUNCTION(BlueprintNativeEvent, CallInEditor, BlueprintCallable, Category = "Selection")
	void ActorSelectionChanged(bool bSelected);

#if WITH_EDITOR
	// ALandscapeBlueprintBrushBase
	virtual void SetOwningLandscape(class ALandscape* InOwningLandscape) override;
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif

private:
	bool bWasSelected = false;

	FDelegateHandle OnActorSelectionChangedHandle;

	/** Called when the editor selection has changed. */
	void HandleActorSelectionChanged(const TArray<UObject*>& NewSelection, bool bForceRefresh);

#if WITH_EDITORONLY_DATA
	//~ This is transient because SetOwningLandscape is called in ALandscape::PostLoad.
	/**
	 * The owning landscape.
	 */
	UPROPERTY(EditAnywhere, Category = Landscape, Transient, meta = (DisplayName = "Landscape"))
	TObjectPtr<ALandscape> DetailPanelLandscape;
#endif

};
