// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class Landmass : ModuleRules
{
	public Landmass(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;
		
		PublicDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"CoreUObject",
				"Engine",
			}
		);

		PrivateDependencyModuleNames.AddRange(
			new string[] {
			}
		);
	}
}