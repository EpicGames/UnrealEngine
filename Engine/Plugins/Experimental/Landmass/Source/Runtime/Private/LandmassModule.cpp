// Copyright Epic Games, Inc. All Rights Reserved.

#include "LandmassModule.h"
#include "Modules/ModuleManager.h"


class FLandmassModule : public ILandmassModuleInterface
{
public:
	virtual void StartupModule() override
	{
	}

	virtual void ShutdownModule() override
	{
	}
};

//////////////////////////////////////////////////////////////////////////

IMPLEMENT_MODULE(FLandmassModule, Landmass);

