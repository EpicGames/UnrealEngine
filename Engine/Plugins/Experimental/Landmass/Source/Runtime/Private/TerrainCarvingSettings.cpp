// Copyright Epic Games, Inc. All Rights Reserved.

#include "TerrainCarvingSettings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TerrainCarvingSettings)

FLandmassTerrainCarvingSettings::FLandmassTerrainCarvingSettings()
	: BlendMode(EBrushBlendType::AlphaBlend)
	, bInvertShape(false)
	, Priority(0)
{


}

