// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "FalloffSettings.generated.h"

UENUM(BlueprintType)
enum class EBrushFalloffMode : uint8
{
	Angle,
	Width,
};

USTRUCT(BlueprintType)
struct FLandmassFalloffSettings
{
	GENERATED_BODY()
	FLandmassFalloffSettings();

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = FalloffSettings)
	EBrushFalloffMode FalloffMode;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = FalloffSettings)
	float FalloffAngle;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = FalloffSettings)
	float FalloffWidth;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = FalloffSettings)
	float EdgeOffset;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = FalloffSettings)
	float ZOffset;
};
