// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleInterface.h"


class ILandmassModuleInterface : public IModuleInterface
{
};
