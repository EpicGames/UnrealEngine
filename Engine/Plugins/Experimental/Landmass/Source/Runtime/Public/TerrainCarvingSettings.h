// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "FalloffSettings.h"
#include "BrushEffectsList.h"
#include "TerrainCarvingSettings.generated.h"

/** The blend mode changes how the brush material is applied to the terrain. */
UENUM(BlueprintType)
enum class EBrushBlendType : uint8
{
	/** Alpha Blend will affect the heightmap both upwards and downwards. */
	AlphaBlend,
	/** Limits the brush to only lowering the terrain. */
	Min,
	/** Limits the brush to only raising the terrain. */
	Max,
	/** Performs an additive blend, using a flat Z=0 terrain as the input. Useful when you want to preserve underlying detail or ramps. */
	Additive,
};

USTRUCT(BlueprintType)
struct FLandmassTerrainCarvingSettings
{
	GENERATED_BODY()

	LANDMASS_API FLandmassTerrainCarvingSettings();

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=TerrainCarvingSettings)
	EBrushBlendType BlendMode;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = TerrainCarvingSettings)
	bool bInvertShape;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = TerrainCarvingSettings)
	FLandmassFalloffSettings FalloffSettings;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = TerrainCarvingSettings)
	FLandmassBrushEffectsList Effects;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = TerrainCarvingSettings)
	int32 Priority;
};
